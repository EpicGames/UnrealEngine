{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Render Trace",
	"Description": "The Render Trace plugin provides a way to have pixel perfect sampling of physical materials on meshes.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "RenderTrace",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"IsExperimentalVersion": false
}
