// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Material shader used to output the dominant physical materials.
 */

#include "/Engine/Private/Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

struct FPhysicalMaterialSamplerInterpolantsVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4 Position : SV_POSITION;
};

#if VERTEXSHADER

/** Simple default vertex shader. */
void VSMain(
	FVertexFactoryInput Input,
	out FPhysicalMaterialSamplerInterpolantsVSToPS Output
)
{
	ResolvedView = ResolveView();
	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);

	half3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);
	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

	WorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);
	ApplyMaterialFirstPersonTransform(VertexParameters, WorldPosition.xyz);
	float4 RasterizedWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, WorldPosition);
	Output.Position = mul(RasterizedWorldPosition, ResolvedView.TranslatedWorldToClip);
}

#elif PIXELSHADER

float DrawSerialNumber;

float2 UpdateDominantMaterial(in float Index, in float Contribution, in float2 DominantMaterial)
{
	if (Contribution > DominantMaterial.y)
	{
		DominantMaterial.x = Index + 1; // Store indices as 1-based so that 0 means empty/default.
		DominantMaterial.y = Contribution;
	}
	return DominantMaterial;
}

/** Pixel shader outputs index of physical material. */
void PSMain(
	FPhysicalMaterialSamplerInterpolantsVSToPS Interpolants
	OPTIONAL_IsFrontFace,
	out HALF4_TYPE OutColor : SV_Target0
)
{
	ResolvedView = ResolveView();

	float4 SvPosition = Interpolants.Position;
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants.FactoryInterpolants, SvPosition);

	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);
	GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);

	float2 DominantMaterial = float2(0, 0); // float2(MaterialIndex, Contribution)

#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 0
	DominantMaterial = UpdateDominantMaterial(0, GetRenderTracePhysicalMaterial0(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 1
	DominantMaterial = UpdateDominantMaterial(1, GetRenderTracePhysicalMaterial1(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 2
	DominantMaterial = UpdateDominantMaterial(2, GetRenderTracePhysicalMaterial2(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 3
	DominantMaterial = UpdateDominantMaterial(3, GetRenderTracePhysicalMaterial3(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 4
	DominantMaterial = UpdateDominantMaterial(4, GetRenderTracePhysicalMaterial4(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 5
	DominantMaterial = UpdateDominantMaterial(5, GetRenderTracePhysicalMaterial5(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 6
	DominantMaterial = UpdateDominantMaterial(6, GetRenderTracePhysicalMaterial6(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 7
	DominantMaterial = UpdateDominantMaterial(7, GetRenderTracePhysicalMaterial7(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 8
	DominantMaterial = UpdateDominantMaterial(8, GetRenderTracePhysicalMaterial8(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 9
	DominantMaterial = UpdateDominantMaterial(9, GetRenderTracePhysicalMaterial9(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 10
	DominantMaterial = UpdateDominantMaterial(10, GetRenderTracePhysicalMaterial10(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 11
	DominantMaterial = UpdateDominantMaterial(11, GetRenderTracePhysicalMaterial11(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 12
	DominantMaterial = UpdateDominantMaterial(12, GetRenderTracePhysicalMaterial12(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 13
	DominantMaterial = UpdateDominantMaterial(13, GetRenderTracePhysicalMaterial13(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 14
	DominantMaterial = UpdateDominantMaterial(14, GetRenderTracePhysicalMaterial14(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 15
	DominantMaterial = UpdateDominantMaterial(15, GetRenderTracePhysicalMaterial15(MaterialParameters), DominantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETRENDERTRACEPHYSICALMATERIAL > 16
	// We pick an arbitrary limit to support here, but the only cost of extending this is more typing.
#error Too many physical materials
#endif

	// Write index assuming R16G16_UNORM
	OutColor = float4(DrawSerialNumber / 65535.0, DominantMaterial.x / 65535.0, 0, 0);
}

#endif
