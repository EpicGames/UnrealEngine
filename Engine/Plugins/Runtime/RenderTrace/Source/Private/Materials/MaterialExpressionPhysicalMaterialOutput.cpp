// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialExpressionPhysicalMaterialOutput.h"

#include "EdGraph/EdGraphNode.h"
#include "MaterialCompiler.h"
#include "Materials/Material.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionPhysicalMaterialOutput)

#define LOCTEXT_NAMESPACE "RenderTrace"

UMaterialExpressionPhysicalMaterialOutput::UMaterialExpressionPhysicalMaterialOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_RenderTrace;
		FConstructorStatics()
			: NAME_RenderTrace(LOCTEXT("RenderTrace", "Render Trace"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_RenderTrace);

	// No outputs
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

const UPhysicalMaterial* UMaterialExpressionPhysicalMaterialOutput::GetInputMaterialFromMap(int32 Index) const
{
	if (Material != nullptr)
	{
		return Material->GetPhysicalMaterialFromMap(Index);
	}
	return nullptr;
}

void UMaterialExpressionPhysicalMaterialOutput::FillMaterialNames()
{
	if (Material)
	{
		TArray<TObjectPtr<class UPhysicalMaterial>> MaterialArray;
		MaterialArray.Empty(Inputs.Num());
		for (auto& Input : Inputs)
		{
			MaterialArray.Add(Input.PhysicalMaterial);
		}
		Material->SetRenderTracePhysicalMaterialOutputs(MaterialArray);
	}
}

void UMaterialExpressionPhysicalMaterialOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString(TEXT("Physical Material Output")));
}

TArrayView<FExpressionInput*> UMaterialExpressionPhysicalMaterialOutput::GetInputsView()
{
	CachedInputs.Empty();
	CachedInputs.Reserve(Inputs.Num());
	for (auto& Input : Inputs)
	{
		CachedInputs.Add(&Input.Input);
	}
	return CachedInputs;
}

FExpressionInput* UMaterialExpressionPhysicalMaterialOutput::GetInput(int32 InputIndex)
{
	return Inputs.IsValidIndex(InputIndex) ? &Inputs[InputIndex].Input : nullptr;
}

FName UMaterialExpressionPhysicalMaterialOutput::GetInputName(int32 InputIndex) const
{
	UPhysicalMaterial* PhysicalMaterial = Inputs[InputIndex].PhysicalMaterial;
	if (PhysicalMaterial != nullptr)
	{
		return PhysicalMaterial->GetFName();
	}
	return NAME_None;
}

int32 UMaterialExpressionPhysicalMaterialOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// In PhysicalMaterialSampler.usf
	if (OutputIndex > 16)
	{
		return CompilerError(Compiler, TEXT("PhysicalMaterialSampler.usf only supports 16 inputs."));
	}

	if (Inputs.IsValidIndex(OutputIndex))
	{
		if (Inputs[OutputIndex].PhysicalMaterial == nullptr)
		{
			return CompilerError(Compiler, TEXT("Physical material not set."));
		}

		if (Inputs[OutputIndex].Input.Expression == nullptr)
		{
			return CompilerError(Compiler, TEXT("Input missing."));
		}

		return Compiler->CustomOutput(this, OutputIndex, Inputs[OutputIndex].Input.Compile(Compiler));
	}

	return INDEX_NONE;
}

void UMaterialExpressionPhysicalMaterialOutput::PostLoad()
{
	Super::PostLoad();
	FillMaterialNames();
}

void UMaterialExpressionPhysicalMaterialOutput::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty)
	{
		const FName PropertyName = PropertyChangedEvent.MemberProperty->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionPhysicalMaterialOutput, Inputs))
		{
			FillMaterialNames();

			if (GraphNode)
			{
				GraphNode->ReconstructNode();
			}
		}
	}
}

#endif // WITH_EDITOR

int32 UMaterialExpressionPhysicalMaterialOutput::GetNumOutputs() const 
{ 
	return Inputs.Num(); 
}

#undef LOCTEXT_NAMESPACE

