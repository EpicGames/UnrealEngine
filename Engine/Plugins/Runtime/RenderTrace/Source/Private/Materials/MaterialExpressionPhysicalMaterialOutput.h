// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "MaterialValueType.h"

#include "MaterialExpressionPhysicalMaterialOutput.generated.h"

#define UE_API RENDERTRACE_API

class UPhysicalMaterial;

/** Structure linking a material expression input with a physical material. For use by UMaterialExpressionPhysicalMaterialOutput. */
USTRUCT()
struct FPhysicalMaterialTraceInput
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = PhysicalMaterial)
	TObjectPtr<UPhysicalMaterial> PhysicalMaterial;

	UPROPERTY(meta = (RequiredInput = "true"))
	FExpressionInput Input;

	FPhysicalMaterialTraceInput()
		: PhysicalMaterial(nullptr)
	{}
};

/** 
 * Custom output node to write out physical material weights.
 */
UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionPhysicalMaterialOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	/** Array of physical material inputs. */
	UPROPERTY(EditAnywhere, Category = PhysicalMaterial)
	TArray<FPhysicalMaterialTraceInput> Inputs;

#if WITH_EDITOR
	UE_API const UPhysicalMaterial* GetInputMaterialFromMap(int32 Index) const;
	UE_API void FillMaterialNames();

	//~ Begin UObject Interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
	UE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	UE_API virtual TArrayView<FExpressionInput*> GetInputsView() override;
	UE_API virtual FExpressionInput* GetInput(int32 InputIndex) override;
	UE_API virtual FName GetInputName(int32 InputIndex) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_Float; }
	UE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	//~ End UMaterialExpression Interface
#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	UE_API virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override { return TEXT("GetRenderTracePhysicalMaterial"); }
	//~ End UMaterialExpressionCustomOutput Interface
};

#undef UE_API
