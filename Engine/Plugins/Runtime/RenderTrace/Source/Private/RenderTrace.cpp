// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTrace.h"

#include "EngineModule.h"
#include "Components/PrimitiveComponent.h"
#include "Materials/Material.h"
#include "Math/OrthoMatrix.h"
#include "MeshPassProcessor.inl"
#include "RenderGraphUtils.h"
#include "RHIGPUReadback.h"
#include "RenderingThread.h"
#include "SceneInterface.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneTexturesConfig.h"
#include "SimpleMeshDrawCommandPass.h"
#include "SceneRendererInterface.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DEFINE_LOG_CATEGORY(LogRenderTrace);

DECLARE_GPU_STAT_NAMED(RenderTraceDraw, TEXT("RenderTrace Draw"));
DECLARE_GPU_STAT_NAMED(RenderTraceReadback, TEXT("RenderTrace Readback"));

DECLARE_STATS_GROUP(TEXT("RenderTrace"), STATGROUP_RenderTrace, STATCAT_Advanced);
DECLARE_DWORD_COUNTER_STAT(TEXT("RenderTrace Requested"), STAT_RenderTrace_Requests, STATGROUP_RenderTrace);
DECLARE_DWORD_COUNTER_STAT(TEXT("RenderTrace Skipped"), STAT_RenderTrace_Skipped, STATGROUP_RenderTrace);
DECLARE_DWORD_COUNTER_STAT(TEXT("RenderTrace Completed"), STAT_RenderTrace_Completed, STATGROUP_RenderTrace);
DECLARE_CYCLE_STAT(TEXT("RenderTrace_Submission"), STAT_RenderTrace_Submit, STATGROUP_RenderTrace);
DECLARE_CYCLE_STAT(TEXT("RenderTrace_Tick"), STAT_RenderTrace_Tick, STATGROUP_RenderTrace);

#define RENDER_TRACE_LOG_TIMING STATS

bool bEnableRenderTracing = false;
FAutoConsoleVariableRef CVarRenderTraceEnable(
	TEXT("RenderTrace.Enabled"),
	bEnableRenderTracing,
	TEXT("Enable Render Tracing."),
	ECVF_Default);


namespace
{
	/** Completion state for a physical material render task. */
	enum class ECompletionState : uint64
	{
		NotStarted = 0,	// Draw not submitted
		Pending = 1,	// Draw submitted, waiting for GPU
		Complete = 2	// Result copied back from GPU
	};
}

struct FPhysicalMaterialMapping
{
	const UPrimitiveComponent* PrimitiveComponent;
	TArray<const UPhysicalMaterial*> PhysicalMaterials;
};

struct FRenderTraceReadbackData
{
	uint16 MaterialMapIndex;
	uint16 PhysicalMaterialIndex;
};
static_assert(sizeof(FRenderTraceReadbackData) == 4, "FRenderTraceReadbackData must match PF_G16R16");

/** Data for a physical material render task. */
struct FRenderTraceTask
{
	// Create on game thread
	TArray<FPhysicalMaterialMapping> MaterialMap;
	uint32 RequestID = 0;
	FIntPoint TargetSize = FIntPoint(ForceInitToZero);
	FVector ViewOrigin = FVector(ForceInitToZero);
	FMatrix ViewRotationMatrix = FMatrix(ForceInitToZero);
	FMatrix ProjectionMatrix = FMatrix(ForceInitToZero);

	// Create on render thread
	TUniquePtr<FRHIGPUTextureReadback> Readback;

	std::atomic<ECompletionState> CompletionState{ ECompletionState::NotStarted };

	// Result reporting
	FRenderTraceDelegate ResultDelegate;
	int64 UserData = 0;
	const UPhysicalMaterial* ResultMaterial = nullptr;

#if RENDER_TRACE_LOG_TIMING
	double StartSeconds = 0;
	uint32 StartFrame = 0;
#endif
};

/** 
 * Get the physical materials attached to the UMaterialExpressionPhysicalMaterialOutput.
 * Returns false if there are no physical materials.
 */
static bool GetPhysicalMaterials(UPrimitiveComponent const* InPrimitiveComponent, TArray<FPhysicalMaterialMapping>& OutMapping)
{
	bool bReturnValue = false;

	int32 NumMaterials = InPrimitiveComponent->GetNumMaterials();
	for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
	{
		UMaterialInterface* PrimitiveMaterial = InPrimitiveComponent->GetMaterial(MaterialIndex);
		if (PrimitiveMaterial == nullptr)
		{
			continue;
		}

		UMaterial* Material = PrimitiveMaterial->GetMaterial();
		if (Material == nullptr)
		{
			continue;
		}

		TArrayView<const TObjectPtr<UPhysicalMaterial>> PhysicalMaterials = Material->GetRenderTracePhysicalMaterialOutputs();
		if (!PhysicalMaterials.IsEmpty())
		{
			FPhysicalMaterialMapping NewMapping;
			NewMapping.PrimitiveComponent = InPrimitiveComponent;
			NewMapping.PhysicalMaterials = PhysicalMaterials;
			OutMapping.Add(MoveTemp(NewMapping));

			bReturnValue = true;
		}
	}

	return bReturnValue;
}


/** Material shader for rendering physical material IDs. */
class FPhysicalMaterialSamplerShader : public FMeshMaterialShader
{
public:
	FPhysicalMaterialSamplerShader()
	{}

	FPhysicalMaterialSamplerShader(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.MaterialParameters.bHasRenderTracePhysicalMaterialOutput && IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

class FPhysicalMaterialSamplerShaderVS : public FPhysicalMaterialSamplerShader
{
	DECLARE_SHADER_TYPE(FPhysicalMaterialSamplerShaderVS, MeshMaterial);

public:
	FPhysicalMaterialSamplerShaderVS()
	{}

	FPhysicalMaterialSamplerShaderVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FPhysicalMaterialSamplerShader(Initializer)
	{
	}
};

//IMPLEMENT_MATERIAL_SHADER_TYPE(, FPhysicalMaterialSamplerShaderVS, TEXT("/Engine/Private/PhysicalMaterialSampler.usf"), TEXT("VSMain"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(, FPhysicalMaterialSamplerShaderVS, TEXT("/Plugin/Runtime/RenderTrace/Private/PhysicalMaterialSampler.usf"), TEXT("VSMain"), SF_Vertex);

struct FPhysicalMaterialSamplerShaderElementData : public FMeshMaterialShaderElementData
{
	FPhysicalMaterialSamplerShaderElementData() : MaterialMapIndex(uint32(-1)) {}

	uint32 MaterialMapIndex;
};

class FPhysicalMaterialSamplerShaderPS : public FPhysicalMaterialSamplerShader
{
	DECLARE_SHADER_TYPE(FPhysicalMaterialSamplerShaderPS, MeshMaterial);

public:
	FPhysicalMaterialSamplerShaderPS()
	{}

	FPhysicalMaterialSamplerShaderPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FPhysicalMaterialSamplerShader(Initializer)
	{
		SerialNumberParameter.Bind(Initializer.ParameterMap, TEXT("DrawSerialNumber"), SPF_Mandatory);
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FPhysicalMaterialSamplerShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		check(ShaderElementData.MaterialMapIndex != -1);
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		ShaderBindings.Add(SerialNumberParameter, ShaderElementData.MaterialMapIndex);
	}

	void GetElementShaderBindings(
		const FShaderMapPointerTable& PointerTable,
		const FScene* Scene,
		const FSceneView* ViewIfDynamicMeshCommand,
		const FVertexFactory* VertexFactory,
		const EVertexInputStreamType InputStreamType,
		const FStaticFeatureLevel FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMeshBatch& MeshBatch,
		const FMeshBatchElement& BatchElement,
		const FPhysicalMaterialSamplerShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FMeshMaterialShader::GetElementShaderBindings(PointerTable, Scene, ViewIfDynamicMeshCommand, VertexFactory, InputStreamType, FeatureLevel, PrimitiveSceneProxy, MeshBatch, BatchElement, ShaderElementData, ShaderBindings, VertexStreams);
	}

	LAYOUT_FIELD(FShaderParameter, SerialNumberParameter);
};

//IMPLEMENT_MATERIAL_SHADER_TYPE(, FPhysicalMaterialSamplerShaderPS, TEXT("/Engine/Private/PhysicalMaterialSampler.usf"), TEXT("PSMain"), SF_Pixel);
IMPLEMENT_MATERIAL_SHADER_TYPE(, FPhysicalMaterialSamplerShaderPS, TEXT("/Plugin/Runtime/RenderTrace/Private/PhysicalMaterialSampler.usf"), TEXT("PSMain"), SF_Pixel);


class FRenderTraceMeshProcessor : public FMeshPassProcessor
{
public:
	FRenderTraceMeshProcessor(
		const FScene* Scene, 
		const FSceneView* InViewIfDynamicMeshCommand, 
		FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId = -1) override final;

	FMeshPassProcessorRenderState PassDrawRenderState;
	uint32 MaterialMapIndex = 0;
};

FRenderTraceMeshProcessor::FRenderTraceMeshProcessor(
	const FScene* Scene, 
	const FSceneView* InViewIfDynamicMeshCommand, 
	FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::Num, Scene, InViewIfDynamicMeshCommand->GetFeatureLevel(), InViewIfDynamicMeshCommand, InDrawListContext)
{
	PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<>::GetRHI());
}

void FRenderTraceMeshProcessor::AddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId)
{
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

			TMeshProcessorShaders<
				FPhysicalMaterialSamplerShaderVS,
				FPhysicalMaterialSamplerShaderPS> PassShaders;

			FMaterialShaderTypes ShaderTypes;
			ShaderTypes.AddShaderType<FPhysicalMaterialSamplerShaderVS>();
			ShaderTypes.AddShaderType<FPhysicalMaterialSamplerShaderPS>();

			FMaterialShaders Shaders;
			if (!Material->TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
			{
				MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
				continue;
			}
			Shaders.TryGetVertexShader(PassShaders.VertexShader);
			Shaders.TryGetPixelShader(PassShaders.PixelShader);

			const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
			const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(*Material, OverrideSettings);
			const ERasterizerCullMode MeshCullMode = CM_None;

			FPhysicalMaterialSamplerShaderElementData ShaderElementData;
			ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, -1, true);
			ShaderElementData.MaterialMapIndex = MaterialMapIndex++;

			const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);

			BuildMeshDrawCommands(
				MeshBatch,
				BatchElementMask,
				PrimitiveSceneProxy,
				*MaterialRenderProxy,
				*Material,
				PassDrawRenderState,
				PassShaders,
				MeshFillMode,
				MeshCullMode,
				SortKey,
				EMeshPassFeatures::Default,
				ShaderElementData);

			break;
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

namespace
{
	/** A description of a mesh to render. */
	struct FMeshInfo
	{
		FPrimitiveSceneProxy const* Proxy;
		FMeshBatch MeshBatch;
		uint64 MeshBatchElementMask;
		uint32 MaterialMapIndex;
	};
	typedef TArray<FMeshInfo, TInlineAllocator<1>> FMeshInfoArray;

	/**
	 * Collect the meshes to render for a component.
	 * WARNING: This gets the SceneProxy pointer from the UComponent on the render thread. This doesn't feel safe but it's what the grass renderer does...
	 */
	void AddMeshInfos_RenderThread(const FPrimitiveSceneProxy* InSceneProxy, FMeshInfoArray& OutMeshInfos, int64 MaterialMappingIndex)
	{
		int32 LODIndex = 0;
		TArray<FMeshBatch> OutMeshElements;
		InSceneProxy->GetMeshDescription(LODIndex, OutMeshElements);

		if (OutMeshElements.Num() > 0)
		{ 
			FMeshInfo& NewMeshInfo = OutMeshInfos.AddDefaulted_GetRef();
			NewMeshInfo.Proxy = InSceneProxy;
			NewMeshInfo.MeshBatch = OutMeshElements[0];
			NewMeshInfo.MeshBatchElementMask = 1 << 0; // LOD 0 only
			NewMeshInfo.MaterialMapIndex = MaterialMappingIndex;
			if (OutMeshElements.Num() > 1)
			{
				UE_LOGF(LogRenderTrace, Warning, "Found a mesh with %d elements, only handling the first one", (int32)OutMeshElements.Num());
			}
		}
	}
	
	BEGIN_SHADER_PARAMETER_STRUCT(FPhysicalMaterialSamplerPassParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	/** Render the physical material IDs and copy to the read back texture. */
	void Render_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		FSceneInterface* SceneInterface,
		FMeshInfoArray const& MeshInfos,
		FIntPoint TargetSize,
		FVector ViewOrigin,
		FMatrix ViewRotationMatrix,
		FMatrix ProjectionMatrix,
		FRHIGPUTextureReadback* Readback)
	{
		FMemMark Mark(FMemStack::Get());
		FRDGBuilder GraphBuilder(RHICmdList);
		
		// Create the view
		FSceneViewFamily::ConstructionValues ViewFamilyInit(nullptr, SceneInterface, FEngineShowFlags(ESFIM_Game));
		FGameTime Time;
		ViewFamilyInit.SetTime(Time);
		FSceneViewFamilyContext ViewFamily(ViewFamilyInit);

		FScenePrimitiveRenderingContextScopeHelper ScenePrimitiveRenderingContextScopeHelper(GetRendererModule().BeginScenePrimitiveRendering(GraphBuilder, &ViewFamily));

		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.SetViewRectangle(FIntRect(0, 0, TargetSize.X, TargetSize.Y));
		ViewInitOptions.ViewOrigin = ViewOrigin;
		ViewInitOptions.ViewRotationMatrix = ViewRotationMatrix;
		ViewInitOptions.ProjectionMatrix = ProjectionMatrix;
		ViewInitOptions.ViewFamily = &ViewFamily;

		// PF_G16R16 Must match FRenderTraceReadbackData
		FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(TargetSize, PF_G16R16, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource),
			TEXT("RenderTraceTarget"));

		FRDGTextureRef DepthTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(TargetSize, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable),
			TEXT("RenderTraceDepthTarget"));

		GetRendererModule().CreateAndInitSingleView(RHICmdList, &ViewFamily, &ViewInitOptions);
		const FSceneView* View = ViewFamily.Views[0];
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View->GPUMask);
			RDG_EVENT_SCOPE_STAT(GraphBuilder, RenderTraceDraw, "RenderTraceRender");

			auto* PassParameters = GraphBuilder.AllocParameters<FPhysicalMaterialSamplerPassParameters>();
			PassParameters->View = View->ViewUniformBuffer;
			PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder, *View);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::EClear);
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);

			AddSimpleMeshPass(GraphBuilder, PassParameters, SceneInterface->GetRenderScene(), *View, nullptr, RDG_EVENT_NAME("RenderTrace"), View->UnscaledViewRect,
				[View, &MeshInfos](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
				{
					FRenderTraceMeshProcessor PassMeshProcessor(nullptr, View, DynamicMeshPassContext);

					for (auto& MeshInfo : MeshInfos)
					{
						const FMeshBatch& Mesh = MeshInfo.MeshBatch;
						if (Mesh.MaterialRenderProxy != nullptr)
						{
							Mesh.MaterialRenderProxy->UpdateUniformExpressionCacheIfNeeded(View->GetFeatureLevel());
							PassMeshProcessor.MaterialMapIndex = MeshInfo.MaterialMapIndex;
							PassMeshProcessor.AddMeshBatch(Mesh, MeshInfo.MeshBatchElementMask, MeshInfo.Proxy);
						}
					}
				});
		}

		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View->GPUMask);
			RDG_EVENT_SCOPE_STAT(GraphBuilder, RenderTraceReadback, "RenderTraceCopy");

			AddEnqueueCopyPass(GraphBuilder, Readback, OutputTexture);
		}

		GraphBuilder.Execute();
	}

	/** Fetch the physical material IDs from a read back texture. */
	void FetchResults_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		FIntPoint TargetSize,
		FRHIGPUTextureReadback* Readback,
		FRenderTraceReadbackData& OutSampleResult)
	{
		int32 Pitch;
		void* Data = Readback->Lock(Pitch);
		check(Data && TargetSize.X <= Pitch);
		if (Data)
		{
			FMemory::Memcpy(&OutSampleResult, Data, sizeof(OutSampleResult));
		}
		Readback->Unlock();
	}

	/** Update the physical material render task on the render thread. */
	void UpdateTask_RenderThread(FRHICommandListImmediate& RHICmdList, FRenderTraceTask& Task, bool bFlush)
	{
		// WARNING: We access the UComponent to get in SceneProxy for AddMeshInfos_RenderThread(). 
		// This isn't good style but probably works since the UComponent owns the update task and is guaranteed to be valid.


		ECompletionState CompletionState = Task.CompletionState.load();
		if (CompletionState == ECompletionState::Pending)
		{
			bool bIsReady = Task.Readback->IsReady();
			if (bFlush || bIsReady)
			{
				if (!bIsReady)
				{
					RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
				}

				FRenderTraceReadbackData Result = {};
				FetchResults_RenderThread(RHICmdList, Task.TargetSize, Task.Readback.Get(), Result);

				if (Task.MaterialMap.IsValidIndex(Result.MaterialMapIndex))
				{
					int32 MaterialIndex = (int32)Result.PhysicalMaterialIndex - 1;
					const TArray<const UPhysicalMaterial*>& PhysicalMaterials = Task.MaterialMap[Result.MaterialMapIndex].PhysicalMaterials;
					if (PhysicalMaterials.IsValidIndex(MaterialIndex))
					{
						Task.ResultMaterial = PhysicalMaterials[MaterialIndex];
					}
					else
					{
						UE_LOGF(LogRenderTrace, Warning, "Completed a task but had an invalid material index (%d:%d)", Result.MaterialMapIndex, Result.PhysicalMaterialIndex);
					}
				}
				else
				{
					UE_LOGF(LogRenderTrace, Warning, "Completed a task but had an invalid material map index (%d:%d)", Result.MaterialMapIndex, Result.PhysicalMaterialIndex);
				}

				Task.CompletionState.store(ECompletionState::Complete);
			}
		}
		else if (CompletionState == ECompletionState::NotStarted)
		{
			if (!Task.Readback.IsValid())
			{
				Task.Readback = MakeUnique<FRHIGPUTextureReadback>(TEXT("PhysicalMaterialSamplerReadback"));
			}

			// Render the pending item.
			FMeshInfoArray MeshInfos;
			FSceneInterface* SceneInterface = nullptr;

			int32 MappingCount = Task.MaterialMap.Num();
			for (int32 MappingIndex = 0; MappingIndex < MappingCount; ++MappingIndex)
			{
				const FPhysicalMaterialMapping& Mapping = Task.MaterialMap[MappingIndex];
				const UPrimitiveComponent* PrimitiveComponent = Mapping.PrimitiveComponent;
				const FPrimitiveSceneProxy* SceneProxy = PrimitiveComponent->SceneProxy;
				if (SceneProxy)
				{
					AddMeshInfos_RenderThread(SceneProxy, MeshInfos, MappingIndex);
					check(SceneInterface == nullptr || SceneInterface == &SceneProxy->GetScene());
					SceneInterface = &SceneProxy->GetScene();
				}
			}

			check(Task.Readback.IsValid());
			Render_RenderThread(
				RHICmdList,
				SceneInterface,
				MeshInfos,
				Task.TargetSize,
				Task.ViewOrigin,
				Task.ViewRotationMatrix,
				Task.ProjectionMatrix,
				Task.Readback.Get());

			Task.CompletionState.store(ECompletionState::Pending);
		}

	}

	void UpdateTasks_RenderThread(FRHICommandListImmediate& RHICmdList, const TArray<TSharedPtr<FRenderTraceTask>>& TaskList, bool bFlush)
	{
		for (const TSharedPtr<FRenderTraceTask>& Task : TaskList)
		{
			check(Task->MaterialMap.Num() > 0);
			UpdateTask_RenderThread(RHICmdList, *Task, bFlush);
		}
	}
}


uint32 FRenderTraceQueue::AsyncRenderTraceComponents(TArrayView<const class UPrimitiveComponent*> PrimitiveComponents, FVector RayOrigin, FVector RayDirection, FRenderTraceDelegate OnComplete, int64 UserData)
{
	if (!ensure(bEnableRenderTracing))
	{
		return 0;
	}

	SCOPE_CYCLE_COUNTER(STAT_RenderTrace_Submit);
	SCOPED_NAMED_EVENT_TEXT("FRenderTraceQueue::AsyncRenderTraceComponents", FColor::Orange);

	INC_DWORD_STAT(STAT_RenderTrace_Requests);
	if (!RayDirection.Normalize())
	{
		UE_LOGF(LogRenderTrace, Warning, "AsyncSampleComponents given invalid RayDirection");
		INC_DWORD_STAT(STAT_RenderTrace_Skipped);
		return 0;
	}

	TArray<FPhysicalMaterialMapping> MaterialMapping;
	for (const UPrimitiveComponent* InPrimitiveComponent : PrimitiveComponents)
	{
		if (ensure(InPrimitiveComponent))
		{
			GetPhysicalMaterials(InPrimitiveComponent, MaterialMapping);
		}
	}

	if (MaterialMapping.IsEmpty())
	{
		INC_DWORD_STAT(STAT_RenderTrace_Skipped);
		return 0;
	}

	uint32 TaskRequestID = ++LastRequestID;

	TSharedPtr<FRenderTraceTask> NewTask = MakeShared<FRenderTraceTask>();
	NewTask->MaterialMap = MoveTemp(MaterialMapping);
	NewTask->CompletionState.store(ECompletionState::NotStarted);
	NewTask->RequestID = TaskRequestID;
	NewTask->ResultDelegate = MoveTemp(OnComplete);
	NewTask->UserData = UserData;

#if STATS
	NewTask->StartSeconds = FPlatformTime::Seconds();
	NewTask->StartFrame = GFrameCounter;
#endif

	const FIntPoint TargetSize(1, 1);
	const FVector TargetCenter = RayOrigin + RayDirection.Normalize();
	const FVector TargetExtent = FVector(TargetSize, 0.0f);

	FVector UpVector = FVector::UpVector;
	if (FMath::IsNearlyEqual(FMath::Abs(FVector::DotProduct(UpVector, RayDirection.GetSafeNormal(SMALL_NUMBER, UpVector))), 1.0f))
	{
		/* If we're too close to being colinear then pick another vector to do the cross products with. */
		UpVector = FVector::RightVector;
	}
	FMatrix ViewMatrix = FLookFromMatrix(FVector::ZeroVector, RayDirection, UpVector);

	const float ZOffset = WORLD_MAX;
	const FMatrix ProjectionMatrix = FReversedZOrthoMatrix(TargetExtent.X, TargetExtent.Y, 0.5f / ZOffset, ZOffset);

	NewTask->TargetSize = TargetSize;
	NewTask->ViewOrigin = TargetCenter;
	NewTask->ViewRotationMatrix = ViewMatrix;
	NewTask->ProjectionMatrix = ProjectionMatrix;

	UE_LOGF(LogRenderTrace, Verbose, "AsyncSampleComponents: Queueing new task %u(%p) with %d primitives", TaskRequestID, NewTask.Get(), PrimitiveComponents.Num());
	RequestsInFlight.Add(MoveTemp(NewTask));
	return TaskRequestID;
}

void FRenderTraceQueue::CancelAsyncSample(uint32 RequestID)
{
	RequestsInFlight.RemoveAll(
		[RequestID](const TSharedPtr<FRenderTraceTask>& Task)
		{
			return Task->RequestID == RequestID;
		});
}

void FRenderTraceQueue::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_RenderTrace_Tick);
	SCOPED_NAMED_EVENT_TEXT("FRenderTraceQueue::Tick", FColor::Orange);

	check(IsInGameThread());
	TArray<TSharedPtr<FRenderTraceTask>> RequestsToUpdate;
	for (TSharedPtr<FRenderTraceTask>& Task : RequestsInFlight)
	{
		if (Task->CompletionState.load() != ECompletionState::Complete)
		{
			RequestsToUpdate.Add(Task);
		}
	}

	if (RequestsToUpdate.Num() > 0)
	{
		UE::RenderCommandPipe::FSyncScope SyncScope;

		ENQUEUE_RENDER_COMMAND(FRenderTraceUpdaterTick)(
			[InRequestsToUpdate=MoveTemp(RequestsToUpdate)](FRHICommandListImmediate& RHICmdList)
			{
				UpdateTasks_RenderThread(RHICmdList, InRequestsToUpdate, false);
			});
	}

	auto CheckCompletionAndRemove = [this](const TSharedPtr<FRenderTraceTask>& Task)
	{
		if (Task->CompletionState.load() == ECompletionState::Complete)
		{
#if RENDER_TRACE_LOG_TIMING
			double TaskDurationSeconds = FPlatformTime::Seconds() - Task->StartSeconds;
			int32 TaskDurationFrames = int32(GFrameCounter - Task->StartFrame);
			UE_LOGF(LogTemp, Log, "RenderTrace completed in %f seconds (%d frames)", TaskDurationSeconds, TaskDurationFrames);
#endif

			INC_DWORD_STAT(STAT_RenderTrace_Completed);
			if (Task->ResultDelegate.IsBound())
			{
				Task->ResultDelegate.Execute(Task->RequestID, Task->ResultMaterial, Task->UserData);
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "RenderTrace completed with an unbound delegate");
			}
			return true;
		}

		return false;
	};

	RequestsInFlight.RemoveAll(CheckCompletionAndRemove);
}

bool FRenderTraceQueue::IsEnabled()
{
	return bEnableRenderTracing;
}

