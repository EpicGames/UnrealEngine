// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTraceModule.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

class FRenderTraceModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	void StartupModule() override;
	void ShutdownModule() override;
};


void FRenderTraceModule::StartupModule()
{
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("RenderTrace"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/Runtime/RenderTrace"), PluginShaderDir);
}

void FRenderTraceModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FRenderTraceModule, RenderTrace);
