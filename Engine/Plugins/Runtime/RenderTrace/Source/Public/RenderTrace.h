// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"
#include "Tickable.h"

#define UE_API RENDERTRACE_API

enum class EStatFlags : uint8;
struct FStatGroup_STATGROUP_Tickables;
struct TStatIdData;

DECLARE_LOG_CATEGORY_EXTERN(LogRenderTrace, Log, All);

DECLARE_DELEGATE_ThreeParams(FRenderTraceDelegate, uint32 TaskID, class UPhysicalMaterial const*, int64 UserData);

/** 
 * The work uses the GPU so this object handles reading back the data without stalling.
 * All calls are expected to be made on the game thread only.
 */
class FRenderTraceQueue : FTickableGameObject
{
public:
	FRenderTraceQueue() = default;
	~FRenderTraceQueue() = default;
	UE_NONCOPYABLE(FRenderTraceQueue);

	/** 
	 *	Initialize the task for a list of components. Previously queued tasks will continue to process. 
	 *	Returns the unique ID of the task that will be sent with the OnCompletion callback, or 0 if the 
	 *  request was invalid or there were no valid primitives to check. 
	 */
	UE_API uint32 AsyncRenderTraceComponents(TArrayView<const class UPrimitiveComponent*> PrimitiveComponents, FVector RayOrigin, FVector RayDirection, FRenderTraceDelegate OnComplete, int64 UserData = 0);
	UE_API void CancelAsyncSample(uint32 RequestID);

	/* FTickableGameObject begin */ 
	UE_API virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FRenderTraceStat, STATGROUP_Tickables); }
	virtual bool IsTickable() const override { return !RequestsInFlight.IsEmpty(); }
	virtual bool IsTickableInEditor() const override { return false; }
	/* FTickableGameObject end */

	static UE_API bool IsEnabled();

private:
	uint32 LastRequestID = 0;
	TArray<TSharedPtr<struct FRenderTraceTask>> RequestsInFlight;
};

#undef UE_API
