// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
* RenderTrace module interface
*/
class IRenderTraceModule : public IModuleInterface
{
public:
};
