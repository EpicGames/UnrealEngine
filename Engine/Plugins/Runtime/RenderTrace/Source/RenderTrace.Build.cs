// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class RenderTrace : ModuleRules
{
	public RenderTrace(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"Projects",
				"RHI",
				"RenderCore", 
				"Renderer",
			}
		);
		SetupModulePhysicsSupport(Target);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"EditorFramework",
					"MaterialUtilities", 
					"Slate",
					"SlateCore",
					"UnrealEd",
				}
			);
		}
	}
}
